{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "DTLS network packet handler",
	"Description" : "Provides a packet handler component to do DTLS encryption and decryption.",
	"Category" : "Misc",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsExperimentalVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name": "DTLSHandlerComponent",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}