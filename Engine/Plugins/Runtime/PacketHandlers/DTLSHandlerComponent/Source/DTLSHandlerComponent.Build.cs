// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DTLSHandlerComponent : ModuleRules
{
    public DTLSHandlerComponent(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "CoreUObject",
				"NetCore",
                "PacketHandler",
                "Engine",
			}
		);

		if (bPlatformSupportsOpenSSL)
		{
			PublicDefinitions.Add("UE_WITH_DTLS=1");

			PublicDependencyModuleNames.Add("SSL");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
		}
		else
		{
			PublicDefinitions.Add("UE_WITH_DTLS=0");
			PrivateDefinitions.Add("WITH_SSL=0");
		}
	}

	protected virtual bool bPlatformSupportsOpenSSL
	{
		get
		{
			return
				Target.Platform == UnrealTargetPlatform.Mac ||
				Target.Platform == UnrealTargetPlatform.Win64 ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) ||
				Target.Platform == UnrealTargetPlatform.IOS ||
				Target.Platform == UnrealTargetPlatform.Android;
		}
	}
}