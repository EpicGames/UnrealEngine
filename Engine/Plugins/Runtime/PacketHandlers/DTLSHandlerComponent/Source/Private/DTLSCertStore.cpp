// Copyright Epic Games, Inc. All Rights Reserved.

#include "DTLSCertStore.h"
#include "DTLSCertificate.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "HAL/IConsoleManager.h"

#if WITH_SSL

#if !UE_BUILD_SHIPPING
static TAutoConsoleVariable<int32> CVarDTLSDebugFingerprints(TEXT("DTLS.DebugFingerprints"), 0, TEXT(""));
#endif

TUniquePtr<FDTLSCertStore> FDTLSCertStore::Instance;

FDTLSCertStore& FDTLSCertStore::Get()
{
	if (!Instance.IsValid())
	{
		Instance = MakeUnique<FDTLSCertStore>();
	}

	return *Instance.Get();
}

TSharedPtr<FDTLSCertificate> FDTLSCertStore::CreateCert(const FTimespan& Lifetime)
{
	TSharedRef<FDTLSCertificate> Cert = MakeShared<FDTLSCertificate>();
	if (Cert->GenerateCertificate(Lifetime))
	{
		return Cert;
	}

	UE_LOGF(LogDTLSHandler, Error, "CreateCert: Failed to create certificate");
	return nullptr;
}

TSharedPtr<FDTLSCertificate> FDTLSCertStore::CreateCert(const FTimespan& Lifetime, const FString& Identifier)
{
	TSharedPtr<FDTLSCertificate> Cert = CreateCert(Lifetime);
	if (Cert.IsValid() && !Identifier.IsEmpty())
	{
		CertMap.Emplace(Identifier, Cert);

#if !UE_BUILD_SHIPPING
		const bool bDebugFingerprints = (CVarDTLSDebugFingerprints.GetValueOnAnyThread() != 0);

		if (bDebugFingerprints)
		{
			FString DebugFilename = FString::Printf(TEXT("%s%s.bin"), *FPaths::ProjectLogDir(), *FPaths::MakeValidFileName(Identifier));
			FFileHelper::SaveArrayToFile(Cert->GetFingerprint(), *DebugFilename);
		}
#endif
	}

	return Cert;
}

TSharedPtr<FDTLSCertificate> FDTLSCertStore::GetCert(const FString& Identifier) const
{
	return CertMap.FindRef(Identifier);
}

TSharedPtr<FDTLSCertificate> FDTLSCertStore::ImportCert(const FString& CertPath) const
{
	TSharedRef<FDTLSCertificate> Cert = MakeShared<FDTLSCertificate>();
	if (Cert->ImportCertificate(CertPath))
	{
		return Cert;
	}

	UE_LOGF(LogDTLSHandler, Error, "ImportCert: Failed to import certificate");
	return nullptr;
}

TSharedPtr<FDTLSCertificate> FDTLSCertStore::ImportCert(const FString& CertPath, const FString& Identifier)
{
	TSharedRef<FDTLSCertificate> Cert = MakeShared<FDTLSCertificate>();
	if (Cert->ImportCertificate(CertPath) && !Identifier.IsEmpty())
	{
		CertMap.Emplace(Identifier, Cert);

		return Cert;
	}

	UE_LOGF(LogDTLSHandler, Error, "ImportCert: Failed to import certificate");
	return nullptr;
}

bool FDTLSCertStore::RemoveCert(const FString& Identifier)
{
	return (CertMap.Remove(Identifier) != 0);
}

#endif // WITH_SSL