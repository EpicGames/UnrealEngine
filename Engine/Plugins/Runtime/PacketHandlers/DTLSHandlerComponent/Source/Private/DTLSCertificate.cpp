// Copyright Epic Games, Inc. All Rights Reserved.

#include "DTLSCertificate.h"
#include "CoreGlobals.h"
#include "Misc/ScopeExit.h"
#include "HAL/FileManager.h"

#if WITH_SSL

#define UI UI_ST
THIRD_PARTY_INCLUDES_START
#include <openssl/asn1.h>
#include <openssl/x509.h>
#include <openssl/bn.h>
#include <openssl/evp.h>
#include <openssl/ossl_typ.h>
#include <openssl/pem.h>
#include <openssl/rsa.h>
THIRD_PARTY_INCLUDES_END
#undef UI


constexpr uint32 FDTLSFingerprint::Length;


FDTLSCertificate::FDTLSCertificate()
	: PKey(nullptr)
	, Certificate(nullptr)
{
}

FDTLSCertificate::~FDTLSCertificate()
{
	FreeCertificate();
}

void FDTLSCertificate::FreeCertificate()
{
	if (Certificate)
	{
		X509_free(Certificate);
		Certificate = nullptr;
	}

	if (PKey)
	{
		EVP_PKEY_free(PKey);
		PKey = nullptr;
	}

	Fingerprint.Reset();
}

bool FDTLSCertificate::ExportCertificate(const FString& CertPath)
{
	if (Certificate)
	{
		if (!CertPath.IsEmpty())
		{
			BIO* CertificateBio = BIO_new(BIO_s_mem());
			PEM_write_bio_X509(CertificateBio, Certificate);

			TArray<uint8> MemBuffer;
			uint64 MemBufferSize = BIO_number_written(CertificateBio);
			MemBuffer.AddUninitialized(MemBufferSize);

			BIO_read(CertificateBio, MemBuffer.GetData(), MemBufferSize);
			BIO_free(CertificateBio);
			CertificateBio = nullptr;

			TUniquePtr<FArchive> ExportAr(IFileManager::Get().CreateFileWriter(*CertPath));

			if (ExportAr.IsValid())
			{
				ExportAr->Serialize(MemBuffer.GetData(), MemBuffer.Num());
				return true;
			}
			else
			{
				UE_LOGF(LogDTLSHandler, Error, "ExportCertificate: Unable to create file writer for: %ls", *CertPath);
			}
		}
		else
		{
			UE_LOGF(LogDTLSHandler, Error, "ExportCertificate: Empty cert path");
		}
	}
	else
	{
		UE_LOGF(LogDTLSHandler, Error, "ExportCertificate: Cert not valid");
	}

	return false;
}

bool FDTLSCertificate::ImportCertificate(const FString& CertPath)
{
	if (Certificate == nullptr)
	{
		if (!CertPath.IsEmpty())
		{
			TUniquePtr<FArchive> ImportAr(IFileManager::Get().CreateFileReader(*CertPath));
			if (ImportAr.IsValid())
			{
				const int64 MemBufferSize = ImportAr->TotalSize();

				TArray<uint8> MemBuffer;
				MemBuffer.AddUninitialized(MemBufferSize + 1);
				ImportAr->Serialize(MemBuffer.GetData(), MemBufferSize);
				MemBuffer[MemBufferSize] = '\0';

				BIO* CertificateBio = BIO_new_mem_buf(MemBuffer.GetData(), -1);
				Certificate = PEM_read_bio_X509(CertificateBio, nullptr, 0, nullptr);

				BIO_free(CertificateBio);
				CertificateBio = nullptr;

				if (Certificate)
				{
					return GenerateFingerprint();
				}
				else
				{
					UE_LOGF(LogDTLSHandler, Error, "ImportCertificate: Unable to read valid x509 certifcate");
				}
			}
			else
			{
				UE_LOGF(LogDTLSHandler, Error, "ImportCertificate: Unable to create file reader for: %ls", *CertPath);
			}
		}
		else
		{
			UE_LOGF(LogDTLSHandler, Error, "ImportCertificate: Empty cert path");
		}
	}
	else
	{
		UE_LOGF(LogDTLSHandler, Error, "ImportCertificate: Cert already exists");
	}

	return false;
}

bool FDTLSCertificate::GenerateCertificate(const FTimespan& Lifetime)
{
	bool bSuccess = false;

	ON_SCOPE_EXIT
	{
		if (!bSuccess)
		{
			FreeCertificate();
		}
	};

	check(!PKey && !Certificate);

	PKey = EVP_PKEY_new();
	Certificate = X509_new();

	// will be freed when the pkey is destroyed
	RSA* RSAKey = RSA_new();
	
	{
		BIGNUM* BigNum = BN_new();
		BN_set_word(BigNum, RSA_F4);
		RSA_generate_key_ex(RSAKey, 2048, BigNum, nullptr);
		BN_free(BigNum);
	}

	EVP_PKEY_assign_RSA(PKey, RSAKey);

	X509_set_version(Certificate, 2);

	ASN1_INTEGER* SerialNumber = ASN1_INTEGER_new();
	
	{
		BIGNUM* BigNumTemp = BN_new();
		BN_rand(BigNumTemp, 159, BN_RAND_TOP_ANY, BN_RAND_BOTTOM_ANY);
		BN_to_ASN1_INTEGER(BigNumTemp, SerialNumber);
		BN_free(BigNumTemp);
	}

	int32 ResultCode = X509_set_serialNumber(Certificate, SerialNumber);
	if (ResultCode == 0)
	{
		UE_LOGF(LogDTLSHandler, Error, "Failed to set serial number on cert: %d", ResultCode);
		return bSuccess;
	}

	ASN1_INTEGER_free(SerialNumber);

	X509_gmtime_adj(X509_get_notBefore(Certificate), 0);
	X509_gmtime_adj(X509_get_notAfter(Certificate), (long)Lifetime.GetTotalSeconds());

	X509_set_pubkey(Certificate, PKey);

	X509_NAME* SubjectName = X509_get_subject_name(Certificate);

	X509_NAME_add_entry_by_txt(SubjectName, "C",  MBSTRING_ASC, (unsigned char *)"US", -1, -1, 0);
	X509_NAME_add_entry_by_txt(SubjectName, "CN", MBSTRING_ASC, (unsigned char *)"Unreal Engine", -1, -1, 0);

	ResultCode = X509_set_issuer_name(Certificate, SubjectName);
	if (ResultCode == 0)
	{
		UE_LOGF(LogDTLSHandler, Error, "Failed to set issuer name");
		return bSuccess;
	}

	ResultCode = X509_sign(Certificate, PKey, EVP_sha256());
	if (ResultCode <= 0)
	{
		UE_LOGF(LogDTLSHandler, Error, "Failed to sign cert: %d", ResultCode);
		return bSuccess;
	}

	bSuccess = GenerateFingerprint();

	return bSuccess;
}

bool FDTLSCertificate::GenerateFingerprint()
{
	Fingerprint.Reset();

	uint32 HashLen = FDTLSFingerprint::Length;

	int32 ResultCode = X509_digest(Certificate, EVP_sha256(), Fingerprint.Data, &HashLen);
	if (ResultCode <= 0)
	{
		UE_LOGF(LogDTLSHandler, Error, "Failed to hash cert: %d", ResultCode);
		return false;
	}

	return true;
}

#endif // WITH_SSL