// Copyright Epic Games, Inc. All Rights Reserved.

#include "DTLSContext.h"
#include "DTLSCertificate.h"
#include "DTLSHandlerComponent.h"
#include "DTLSCertStore.h"

#if WITH_SSL

#define UI UI_ST
THIRD_PARTY_INCLUDES_START
#include "Interfaces/ISslCertificateManager.h"
#include <openssl/ssl.h>
#include "SslModule.h"
#include <openssl/bio.h>
#include <openssl/dtls1.h>
#include <openssl/evp.h>
#include <openssl/ossl_typ.h>
#include <openssl/x509.h>
#include <openssl/x509_vfy.h>
THIRD_PARTY_INCLUDES_END
#undef UI

namespace DTLSContext
{
	static const char* CipherListPSK = "PSK-AES256-GCM-SHA384";
	static const char* CipherListCert = "HIGH";

	TAutoConsoleVariable<int32> CVarCertLifetime(TEXT("DTLS.CertLifetime"), 4 * 60 * 60, TEXT("Lifetime to set on generated certificates, in seconds."));
	TAutoConsoleVariable<int32> CVarHandshakeRetry(TEXT("DTLS.HandshakeRetry"), 500, TEXT("Handshake retry time, in milliseconds."));
}

const TCHAR* LexToString(EDTLSContextType ContextType)
{
	switch (ContextType)
	{
	case EDTLSContextType::Server:
		return TEXT("Server");
	case EDTLSContextType::Client:
		return TEXT("Client");
	default:
		return TEXT("Unknown");
	}
}

static unsigned int DTLSTimerCallback(SSL* SSlPtr, unsigned int TimerInUS)
{
	return DTLSContext::CVarHandshakeRetry.GetValueOnAnyThread() * 1000;
}

static int DTLSCertVerifyCallback(X509_STORE_CTX* Context, void* UserData)
{
	FDTLSContext* DTLSContext = static_cast<FDTLSContext*>(UserData);

	X509* RemoteCert = X509_STORE_CTX_get0_cert(Context);
	if (RemoteCert)
	{
		int32 CmpTimeResult = X509_cmp_current_time(X509_get_notBefore(RemoteCert));
		if (CmpTimeResult >= 0)
		{
			UE_LOGF(LogDTLSHandler, Error, "DTLSCertVerifyCallback: Remote certificate times are not valid.");
			return 0;
		}

		CmpTimeResult = X509_cmp_current_time(X509_get_notAfter(RemoteCert));
		if (CmpTimeResult <= 0)
		{
			UE_LOGF(LogDTLSHandler, Error, "DTLSCertVerifyCallback: Remote certificate times are not valid.");
			return 0;
		}

		// check subject name == issuer name (self-signed)
		X509_NAME* IssuerName = X509_get_issuer_name(RemoteCert);
		X509_NAME* SubjectName = X509_get_subject_name(RemoteCert);

		if (X509_NAME_cmp(IssuerName, SubjectName) != 0)
		{
			UE_LOGF(LogDTLSHandler, Error, "DTLSCertVerifyCallback: Remote certificate is not self signed.");
			return 0;
		}

		// only checking in clients for now
		if (DTLSContext && (DTLSContext->GetContextType() == EDTLSContextType::Client))
		{
			// check against expected fingerprint
			FDTLSFingerprint Fingerprint;
			uint32 HashLen = FDTLSFingerprint::Length;

			int32 ResultCode = X509_digest(RemoteCert, EVP_sha256(), Fingerprint.Data, &HashLen);
			if (ResultCode <= 0)
			{
				UE_LOGF(LogDTLSHandler, Error, "DTLSCertVerifyCallback: Unable to compute fingerprint");
				return 0;
			}

			if (SSL* SSLPtr = static_cast<SSL*>(X509_STORE_CTX_get_ex_data(Context, SSL_get_ex_data_X509_STORE_CTX_idx())))
			{
				if (const FDTLSHandlerComponent* Handler = static_cast<FDTLSHandlerComponent*>(SSL_get_app_data(SSLPtr)))
				{
					if (const FDTLSFingerprint* RemoteFingerprint = Handler->GetRemoteFingerprint())
					{
						if (FMemory::Memcmp(Fingerprint.Data, RemoteFingerprint->Data, HashLen) != 0)
						{
							UE_LOGF(LogDTLSHandler, Error, "DTLSCertVerifyCallback: Fingerprint validation failure");
							return 0;
						}
					}
					else
					{
						UE_LOGF(LogDTLSHandler, Error, "DTLSCertVerifyCallback: Unable to retrieve remote fingerprint");
						return 0;
					}
				}
				else
				{
					UE_LOGF(LogDTLSHandler, Error, "DTLSCertVerifyCallback: Unable to retrieve handler");
					return 0;
				}
			}
			else
			{
				UE_LOGF(LogDTLSHandler, Error, "DTLSCertVerifyCallback: Unable to retrieve ssl context");
				return 0;
			}
		}
	}
	else
	{
		UE_LOGF(LogDTLSHandler, Error, "DTLSCertVerifyCallback: Unable to retrieve remote cert");
		return 1;
	}
	
	UE_LOGF(LogDTLSHandler, Log, "DTLSCertVerifyCallback: Verified");
	return 1;
}

static void DTLSInfoCallback(const SSL* SSLPtr, int InfoType, int Value)
{
	UE_LOGF(LogDTLSHandler, Verbose, "SSL Info State: %ls", ANSI_TO_TCHAR(SSL_state_string_long(SSLPtr)));
	
	if (InfoType & SSL_CB_ALERT)
	{
		UE_LOGF(LogDTLSHandler, Warning, "    Alert: %ls Desc: %ls", ANSI_TO_TCHAR(SSL_alert_type_string_long(Value)), ANSI_TO_TCHAR(SSL_alert_desc_string_long(Value)));
	}
}

static unsigned int DTLSPSKClientCallback(SSL* SSLPtr, const char* Hint, char* Identity, unsigned int MaxIdentityLength, unsigned char* PSK, unsigned int MaxPSKLength)
{
	if (FDTLSHandlerComponent* Handler = static_cast<FDTLSHandlerComponent*>(SSL_get_app_data(SSLPtr)))
	{
		if (const FDTLSPreSharedKey* PreSharedKey = Handler->GetPreSharedKey())
		{
			const FString& PSKIdentity = PreSharedKey->GetIdentity();
			if (!PSKIdentity.IsEmpty())
			{
				auto IdentityAnsi = StringCast<ANSICHAR>(*PSKIdentity);
				int32 IdentityLen = FCStringAnsi::Strlen(IdentityAnsi.Get()) + 1;

				check(IdentityLen <= (int32)MaxIdentityLength);
				FCStringAnsi::Strncpy(Identity, IdentityAnsi.Get(), IdentityLen);
			}

			TArrayView<const uint8> KeyView = PreSharedKey->GetKey();

			check(KeyView.Num() <= (int32)MaxPSKLength);
			FMemory::Memcpy(PSK, KeyView.GetData(), KeyView.Num());

			UE_LOGF(LogDTLSHandler, Log, "DTLSPSKClientCallback: Key successfully set, Identity: %ls", ANSI_TO_TCHAR(Identity));
		
			return KeyView.Num();
		}
		else
		{
			UE_LOGF(LogDTLSHandler, Error, "DTLSPSKClientCallback:  Invalid key");
		}
	}
	else
	{
		UE_LOGF(LogDTLSHandler, Error, "DTLSPSKClientCallback:  Invalid owner");
	}

	return 0;
}

static unsigned int DTLSPSKServerCallback(SSL* SSLPtr, const char* Identity, unsigned char* PSK, unsigned int MaxPSKLength)
{
	if (FDTLSHandlerComponent* Handler = static_cast<FDTLSHandlerComponent*>(SSL_get_app_data(SSLPtr)))
	{
		if (const FDTLSPreSharedKey* PreSharedKey = Handler->GetPreSharedKey())
		{
			const FString& PSKIdentity = PreSharedKey->GetIdentity();

			if (!PSKIdentity.IsEmpty() && Identity)
			{
				auto IdentityAnsi = StringCast<ANSICHAR>(*PSKIdentity);

				if (FCStringAnsi::Strcmp(Identity, IdentityAnsi.Get()) != 0)
				{
					UE_LOGF(LogDTLSHandler, Error, "DTLSPSKServerCallback: Unexpected identity: %ls Expected: %ls", ANSI_TO_TCHAR(Identity), *PSKIdentity);
					return 0;
				}
			}

			TArrayView<const uint8> KeyView = PreSharedKey->GetKey();

			check(KeyView.Num() <= (int32)MaxPSKLength);
			FMemory::Memcpy(PSK, KeyView.GetData(), KeyView.Num());

			UE_LOGF(LogDTLSHandler, Log, "DTLSPSKServerCallback: Key successfully set");

			return KeyView.Num();
		}
		else
		{
			UE_LOGF(LogDTLSHandler, Error, "DTLSPSKServerCallback:  Invalid key");
		}
	}
	else
	{
		UE_LOGF(LogDTLSHandler, Error, "DTLSPSKServerCallback:  Invalid owner");
	}

	return 0;
}

int FMTUFilter::Create(BIO* BIOPtr) 
{
	FMTUFilter* MTUFilter = new FMTUFilter();

	BIO_set_init(BIOPtr, 1);
	BIO_set_data(BIOPtr, MTUFilter);
	BIO_set_flags(BIOPtr, 0);

 	return 1;
}

int FMTUFilter::Destroy(BIO* BIOPtr) 
{
	if (BIOPtr == nullptr)
	{
		return 0;
	}

	FMTUFilter* MTUFilter = static_cast<FMTUFilter*>(BIO_get_data(BIOPtr));
	if (MTUFilter != nullptr) 
	{
		delete MTUFilter;
	}

	BIO_set_data(BIOPtr, nullptr);
	BIO_set_init(BIOPtr, 0);
	BIO_set_flags(BIOPtr, 0);

	return 1;
}

int FMTUFilter::Write(BIO* BIOPtr, const char* Data, int DataLen) 
{
	check(IsInGameThread());

	int WriteResult = BIO_write(BIO_next(BIOPtr), Data, DataLen);

	FMTUFilter* MTUFilter = static_cast<FMTUFilter*>(BIO_get_data(BIOPtr));
	if (MTUFilter != nullptr) 
	{
		MTUFilter->FragmentSizes.Add(WriteResult);
	}

	return WriteResult;
}

long FMTUFilter::Ctrl(BIO* BIOPtr, int CtrlCommand, long, void*)
{
	check(IsInGameThread());

	FMTUFilter* MTUFilter = static_cast<FMTUFilter*>(BIO_get_data(BIOPtr));

	switch (CtrlCommand) 
	{
	case BIO_CTRL_FLUSH:
		return 1;
	case BIO_CTRL_DGRAM_QUERY_MTU:
		ensureMsgf(0, TEXT("Filter BIO received an MTU query, this should have been disabled on the context."));
		return 0;
	case BIO_CTRL_DGRAM_GET_MTU_OVERHEAD:
		return 0;
	case BIO_CTRL_WPENDING:
		return 0;
	case BIO_CTRL_PENDING: 
	{
		if (MTUFilter == nullptr || MTUFilter->FragmentSizes.Num() == 0)
		{
			return 0;
		}

		long FragmentSize = MTUFilter->FragmentSizes[0];
		MTUFilter->FragmentSizes.RemoveAt(0);
		return FragmentSize;
	}
	case BIO_CTRL_PUSH:
	case BIO_CTRL_POP:
		return 0;
	default:
		UE_LOGF(LogDTLSHandler, Warning, "FMTUFilter::Ctrl Unhandled command: %d", CtrlCommand);
		break;
	}

	return 0;
}

FDTLSContext::FDTLSContext(EDTLSContextType InContextType)
	: ContextType(InContextType)
	, SSLContext(nullptr)
	, SSLPtr(nullptr)
	, InBIO(nullptr)
	, OutBIO(nullptr)
	, MTUFilterMethod(nullptr)
{
}

FDTLSContext::~FDTLSContext()
{
	if (MTUFilterMethod)
	{
		BIO_meth_free(MTUFilterMethod);
	}

	// bios are freed by SSL_free

	if (SSLPtr)
	{
		SSL_free(SSLPtr);
	}

	if (SSLContext)
	{
		SSL_CTX_free(SSLContext);
	}
}

bool FDTLSContext::Initialize(const int32 MaxPacketSize, const FString& CertId, FDTLSHandlerComponent* Handler)
{
	SSLContext = SSL_CTX_new(DTLS_method());
	if (!SSLContext)
	{
		UE_LOGF(LogDTLSHandler, Error, "Failed to create SSL context.");
		return false;
	}

	SSL_CTX_set_min_proto_version(SSLContext, DTLS1_2_VERSION);

	FSslModule::Get().GetCertificateManager().AddCertificatesToSslContext(SSLContext);

	const bool bPreSharedKeys = (CVarPreSharedKeys.GetValueOnAnyThread() != 0);

	const char* CipherList = bPreSharedKeys ? DTLSContext::CipherListPSK : DTLSContext::CipherListCert;

	int32 Result = SSL_CTX_set_cipher_list(SSLContext, CipherList);
	if (Result == 0) 
	{
		UE_LOGF(LogDTLSHandler, Error, "Failed to set cipher list on SSL context: %d", Result);
		return false;
	}

	SSL_CTX_set_app_data(SSLContext, this);

	SSL_CTX_set_mode(SSLContext, SSL_MODE_AUTO_RETRY); // should be on by default in 1.1.1+ ?

	SSL_CTX_set_options(SSLContext, SSL_OP_NO_QUERY_MTU);
	SSL_CTX_set_options(SSLContext, SSL_OP_CIPHER_SERVER_PREFERENCE);

	if (bPreSharedKeys)
	{
		SSL_CTX_set_psk_client_callback(SSLContext, DTLSPSKClientCallback);
		SSL_CTX_set_psk_server_callback(SSLContext, DTLSPSKServerCallback);
	}
	else
	{
		SSL_CTX_set_verify(SSLContext, SSL_VERIFY_PEER | SSL_VERIFY_FAIL_IF_NO_PEER_CERT, nullptr);
		SSL_CTX_set_cert_verify_callback(SSLContext, DTLSCertVerifyCallback, this);

		if (!CertId.IsEmpty())
		{
			Cert = FDTLSCertStore::Get().GetCert(CertId);
		}
		else
		{
			UE_LOGF(LogDTLSHandler, Warning, "Empty certificate identifier");

			FTimespan Lifetime = FTimespan::FromSeconds(DTLSContext::CVarCertLifetime.GetValueOnAnyThread());

			Cert = FDTLSCertStore::Get().CreateCert(Lifetime);
		}

		if (Cert.IsValid())
		{
			SSL_CTX_use_certificate(SSLContext, Cert->GetCertificate());
			SSL_CTX_use_PrivateKey(SSLContext, Cert->GetPKey());
			
			Result = SSL_CTX_check_private_key(SSLContext);
			if (Result != 1)
			{
				UE_LOGF(LogDTLSHandler, Error, "Private key check failed: %d", Result);
				return false;
			}
		}
		else
		{
			UE_LOGF(LogDTLSHandler, Error, "Failed to retrieve certificate");
			return false;
		}
	}

	SSLPtr = SSL_new(SSLContext);
	if (!SSLPtr)
	{
		UE_LOGF(LogDTLSHandler, Error, "Failed to create SSL session.");
		return false;
	}

	SSL_set_app_data(SSLPtr, Handler);
	SSL_set_info_callback(SSLPtr, DTLSInfoCallback);
	SSL_set_mtu(SSLPtr, MaxPacketSize);

	MTUFilterMethod = BIO_meth_new(BIO_TYPE_FILTER, "MTUFilter");
	if (!MTUFilterMethod)
	{
		UE_LOGF(LogDTLSHandler, Error, "Failed to create SSL MTU filter.");
		return false;
	}

	BIO_meth_set_create(MTUFilterMethod, &FMTUFilter::Create);
	BIO_meth_set_destroy(MTUFilterMethod, &FMTUFilter::Destroy);
	BIO_meth_set_write(MTUFilterMethod, &FMTUFilter::Write);
	BIO_meth_set_ctrl(MTUFilterMethod, &FMTUFilter::Ctrl);

	InBIO = BIO_new(BIO_s_mem());
	OutBIO = BIO_new(BIO_s_mem());
	FilterBIO = BIO_new(MTUFilterMethod);

	BIO_push(FilterBIO, OutBIO);
	SSL_set_bio(SSLPtr, InBIO, FilterBIO);

	if (ContextType == EDTLSContextType::Server)
	{
		SSL_set_accept_state(SSLPtr);
	}
	else
	{
		SSL_set_connect_state(SSLPtr);
	}

	DTLS_set_timer_cb(SSLPtr, DTLSTimerCallback);

	return true;
}

bool FDTLSContext::IsHandshakeComplete() const
{
	return (SSLPtr != nullptr) && SSL_is_init_finished(SSLPtr);
}


#endif // WITH_SSL