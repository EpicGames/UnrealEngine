// Copyright Epic Games, Inc. All Rights Reserved.

#include "DTLSHandlerComponent.h"
#include "Modules/ModuleManager.h"

#if WITH_SSL

#define UI UI_ST
THIRD_PARTY_INCLUDES_START
#include "DTLSCertificate.h"
#include <openssl/ssl.h>
#include "DTLSContext.h"
#include <openssl/dtls1.h>
#include "Interfaces/ISslManager.h"
#include <openssl/err.h>
#include "SslModule.h"
#include "Stats/Stats.h"
#include <openssl/bio.h>
#include <openssl/crypto.h>
#include <openssl/evp.h>
#include <openssl/ossl_typ.h>
THIRD_PARTY_INCLUDES_END
#undef UI

TAutoConsoleVariable<int32> CVarPreSharedKeys(TEXT("DTLS.PreSharedKeys"), 1, TEXT("If non-zero, use pre-shared keys, otherwise self-signed certificates will be generated."));

#endif // WITH_SSL

DEFINE_LOG_CATEGORY(LogDTLSHandler);

IMPLEMENT_MODULE(FDTLSHandlerComponentModule, DTLSHandlerComponent)

void FDTLSHandlerComponentModule::StartupModule()
{
	FPacketHandlerComponentModuleInterface::StartupModule();

#if WITH_SSL
	FSslModule& SslModule = FModuleManager::LoadModuleChecked<FSslModule>("SSL");
	if (!SslModule.GetSslManager().InitializeSsl())
	{
		SSL_library_init();
		SSL_load_error_strings();        
		OpenSSL_add_all_algorithms();
	}
#endif // WITH_SSL
}

void FDTLSHandlerComponentModule::ShutdownModule()
{
#if WITH_SSL
	FSslModule& SslModule = FModuleManager::LoadModuleChecked<FSslModule>("SSL");
	SslModule.GetSslManager().ShutdownSsl();
#endif // WITH_SSL

	FPacketHandlerComponentModuleInterface::ShutdownModule();
}

TSharedPtr<HandlerComponent> FDTLSHandlerComponentModule::CreateComponentInstance(FString& Options)
{
#if WITH_SSL
	TSharedRef<HandlerComponent> ReturnVal = MakeShared<FDTLSHandlerComponent>();

	return ReturnVal;
#else
	return nullptr;
#endif // WITH_SSL
}

#if WITH_SSL
FDTLSHandlerComponent::FDTLSHandlerComponent()
	: FEncryptionComponent(FName(TEXT("DTLSHandlerComponent")))
	, InternalState(EDTLSHandlerState::Unencrypted)
	, bPendingHandshakeData(false)
{
}

FDTLSHandlerComponent::~FDTLSHandlerComponent()
{
}

void FDTLSHandlerComponent::SetEncryptionData(const FEncryptionData& EncryptionData)
{
	const bool bPreSharedKeys = (CVarPreSharedKeys.GetValueOnAnyThread() != 0);

	if (bPreSharedKeys)
	{
		PreSharedKey = MakeUnique<FDTLSPreSharedKey>();
		PreSharedKey->SetPreSharedKey(EncryptionData.Key);
		PreSharedKey->SetIdentity(EncryptionData.Identifier);
	}
	else
	{
		CertId = EncryptionData.Identifier;

		if (Handler->Mode == UE::Handler::Mode::Client)
		{
			RemoteFingerprint = MakeUnique<FDTLSFingerprint>();

			if (EncryptionData.Fingerprint.Num() == FDTLSFingerprint::Length)
			{
				FMemory::Memcpy(RemoteFingerprint->Data, EncryptionData.Fingerprint.GetData(), EncryptionData.Fingerprint.Num());
			}
			else
			{
				UE_LOGF(LogDTLSHandler, Error, "SetEncryptionData: Invalid fingerprint size: got %d expected %d", EncryptionData.Fingerprint.Num(), FDTLSFingerprint::Length);
			}
		}
	}
}

void FDTLSHandlerComponent::EnableEncryption()
{
	EDTLSContextType ContextType = (Handler->Mode == UE::Handler::Mode::Server) ? EDTLSContextType::Server : EDTLSContextType::Client;

	UE_LOGF(LogDTLSHandler, Log, "EnableEncryption: %ls", LexToString(ContextType));

	DTLSContext = MakeUnique<FDTLSContext>(ContextType);
	if (DTLSContext.IsValid())
	{
		check(Handler);

		const int32 MaxPacketSize = (MaxOutgoingBits - GetReservedPacketBits()) / 8;

		if (DTLSContext->Initialize(MaxPacketSize, CertId, this))
		{
			// clients should begin handshaking immediately
			// servers should wait until they receive the first handshaking message
			if (ContextType == EDTLSContextType::Client)
			{
				InternalState = EDTLSHandlerState::Handshaking;
			}
		}
		else
		{
			UE_LOGF(LogDTLSHandler, Error, "EnableEncryption: Failed to initialize context.");
			DTLSContext.Reset();
		}
	}
	else
	{
		UE_LOGF(LogDTLSHandler, Error, "EnableEncryption: Invalid context.");
	}
}

void FDTLSHandlerComponent::DisableEncryption()
{
	UE_LOGF(LogDTLSHandler, Log, "DisableEncryption");

	DTLSContext.Reset();
}

bool FDTLSHandlerComponent::IsEncryptionEnabled() const
{
	return DTLSContext.IsValid();
}

void FDTLSHandlerComponent::Initialize()
{
	SetActive(true);
	SetState(UE::Handler::Component::State::Initialized);
	Initialized();
}

bool FDTLSHandlerComponent::IsValid() const
{
	return true;
}

void FDTLSHandlerComponent::Incoming(FBitReader& Packet)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("PacketHandler DTLS Decrypt"), STAT_PacketHandler_DTLS_Decrypt, STATGROUP_Net);

	// check encrypted bit
	if (Packet.ReadBit() != 0)
	{
		if (DTLSContext.IsValid())
		{
			const int32 HandshakeBit = Packet.ReadBit();
			const int32 PayloadBytes = Packet.GetBytesLeft();

			if (PayloadBytes > 0 && PayloadBytes <= sizeof(TempBuffer))
			{
				TempBuffer[PayloadBytes - 1] = 0;
			}
			else
			{
				UE_LOGF(LogDTLSHandler, Log, "FDTLSHandlerComponent::Incoming: invalid payload size");
				Packet.SetError();
				return;
			}

			Packet.SerializeBits(TempBuffer, Packet.GetBitsLeft());

			if (HandshakeBit == 1)
			{
				// feed the incoming data to the context
				const int32 BytesWritten = BIO_write(DTLSContext->GetInBIO(), TempBuffer, PayloadBytes);
				if (BytesWritten != PayloadBytes)
				{
					UE_LOGF(LogDTLSHandler, Warning, "Failed to write entire incoming packet to input BIO: %d / %d", BytesWritten, PayloadBytes);
					Packet.SetError();
					return;
				}

				if (InternalState == EDTLSHandlerState::Unencrypted)
				{
					// server can move to handshaking now
					check(Handler->Mode == UE::Handler::Mode::Server);
					InternalState = EDTLSHandlerState::Handshaking;
				}

				if (InternalState != EDTLSHandlerState::Handshaking)
				{
					bPendingHandshakeData = true;
				}
			}
			else
			{
				if (InternalState == EDTLSHandlerState::Handshaking)
				{
					UE_LOGF(LogDTLSHandler, Verbose, "Ignoring non-handshake packet while handshake is still in progress.");
					Packet.SetData(nullptr, 0);
					return;
				}
				else if (InternalState == EDTLSHandlerState::Encrypted)
				{
					const int32 BytesWritten = BIO_write(DTLSContext->GetInBIO(), TempBuffer, PayloadBytes);
					if (BytesWritten == PayloadBytes)
					{
						const int32 BytesRead = SSL_read(DTLSContext->GetSSLPtr(), TempBuffer, sizeof(TempBuffer));
						if (BytesRead > 0)
						{
							// Look for the termination bit that was written in Outgoing() to determine the exact bit size.
							uint8 LastByte = TempBuffer[BytesRead - 1];
							if (LastByte != 0)
							{
								int32 BitSize = (BytesRead * 8) - 1;

								// Bit streaming, starts at the Least Significant Bit, and ends at the MSB.
								while (!(LastByte & 0x80))
								{
									LastByte *= 2;
									BitSize--;
								}

								Packet.SetData(TempBuffer, BitSize);
							}
							else
							{
								UE_LOGF(LogTemp, Log, "DTLS Error");
								Packet.SetError();
							}
						}
						else 
						{
							int32 SSLError = SSL_get_error(DTLSContext->GetSSLPtr(), BytesRead);

							if (SSLError == SSL_ERROR_WANT_READ)
							{
								UE_LOGF(LogDTLSHandler, Verbose, "Ignoring non-handshake packet due to pending handshake data.");
								Packet.SetData(nullptr, 0);
								return;
							}
							else
							{
								UE_LOGF(LogDTLSHandler, Error, "SSL_read error: %d", SSLError);
								Packet.SetError();
							}
						}
					}
					else
					{
						UE_LOGF(LogDTLSHandler, Error, "Failed to write entire incoming packet to input BIO: %d / %d", BytesWritten, PayloadBytes);
						Packet.SetError();
					}
				}
				else
				{
					UE_LOGF(LogDTLSHandler, Error, "Attempted to process packet with handler in invalid state");
					Packet.SetError();
				}
			}
		}
		else
		{
			UE_LOGF(LogDTLSHandler, Verbose, "Ignoring encrypted packet before handler has been enabled.");
			Packet.SetData(nullptr, 0);
		}
	}
}

void FDTLSHandlerComponent::DoHandshake()
{
	SSL* SSLPtr = DTLSContext->GetSSLPtr();

	const int32 HandshakeResult = SSL_do_handshake(SSLPtr);
	if (HandshakeResult == 0)
	{
		LogError(TEXT("SSL_do_handshake"), HandshakeResult);
		DTLSContext.Reset();
		return;
	}
	else if (HandshakeResult < 0)
	{
		int SSLError = SSL_get_error(SSLPtr, HandshakeResult);

		if (SSLError != SSL_ERROR_NONE && SSLError != SSL_ERROR_WANT_READ && SSLError != SSL_ERROR_WANT_WRITE)
		{
			UE_LOGF(LogDTLSHandler, Error, "DoHandshake:  Handshaking failed with result: %d error: %d", HandshakeResult, SSLError);
			DTLSContext.Reset();
			return;
		}
	}

	struct FDTLSTimeout
	{
		long sec;
		long usec;
	} TimeLeft;

	const int32 GetTimeoutResult = DTLSv1_get_timeout(SSLPtr, &TimeLeft);
	if (GetTimeoutResult != 0)
	{
		UE_LOGF(LogDTLSHandler, VeryVerbose, "DoHandshake:  Timeout: %ld / %ld", TimeLeft.sec, TimeLeft.usec);

		if (TimeLeft.sec <= 0 && TimeLeft.usec <= 0)
		{
			UE_LOGF(LogDTLSHandler, Log, "DoHandshake:  Handshaking timeout occurred, retrying.");

			const int32 HandleTimeoutResult = DTLSv1_handle_timeout(SSLPtr);
			if (HandleTimeoutResult < 0)
			{
				LogError(TEXT("DTLSv1_handle_timeout"), HandleTimeoutResult);
				DTLSContext.Reset();
				return;
			}
		}
	}

	int32 Pending = BIO_ctrl_pending(DTLSContext->GetFilterBIO());
	while (Pending > 0)
	{
		if (Pending <= sizeof(TempBuffer))
		{
			const int32 BytesRead = BIO_read(DTLSContext->GetOutBIO(), TempBuffer, Pending);
			if (BytesRead > 0)
			{
				if (BytesRead == Pending)
				{
					FBitWriter OutPacket(0, true);
					OutPacket.WriteBit(1);	// encryption enabled
					OutPacket.WriteBit(1);	// handshake packet
					OutPacket.SerializeBits(TempBuffer, BytesRead * 8);

					UE_LOGF(LogDTLSHandler, Verbose, "DoHandshake:  Sending handshake packet: %d bytes", BytesRead);

					// SendHandlerPacket is a low level send and is not reliable
					FOutPacketTraits Traits;
					Handler->SendHandlerPacket(this, OutPacket, Traits);
				}
				else
				{
					DTLSContext.Reset();
					UE_LOGF(LogDTLSHandler, Error, "Unexpected value from BIO_read: Read: %d Pending: %d", BytesRead, Pending);
					return;
				}
			}
			else
			{
				UE_LOGF(LogDTLSHandler, Error, "BIO_read error: %d", BytesRead);
				return;
			}
		}
		else
		{
			DTLSContext.Reset();
			UE_LOGF(LogDTLSHandler, Error, "BIO_ctrl_pending returned value out of range: %d", Pending);
			return;
		}

		Pending = BIO_ctrl_pending(DTLSContext->GetFilterBIO());
	}
}

void FDTLSHandlerComponent::TickHandshake()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("PacketHandler DTLS Handshake"), STAT_PacketHandler_DTLS_Handshake, STATGROUP_Net);

	if ((InternalState == EDTLSHandlerState::Handshaking) || bPendingHandshakeData)
	{
		// always process handshake data so there won't be anything left in the internal buffers when we change states
		if (DTLSContext.IsValid())
		{
			DoHandshake();
		}
		
		// DoHandshake could cause the context to become invalid, so check again
		if (DTLSContext.IsValid() && DTLSContext->IsHandshakeComplete() && (InternalState == EDTLSHandlerState::Handshaking))
		{
			InternalState = EDTLSHandlerState::Encrypted;
			UE_LOGF(LogDTLSHandler, Log, "UpdateHandshake:  Handshaking completed");
		}

		bPendingHandshakeData = false;
	}
}

void FDTLSHandlerComponent::Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("PacketHandler DTLS Encrypt"), STAT_PacketHandler_DTLS_Encrypt, STATGROUP_Net);

	const bool bEncryptionEnabled = (InternalState != EDTLSHandlerState::Unencrypted);

	FBitWriter NewPacket(Packet.GetNumBits() + 2, true);
	NewPacket.WriteBit(bEncryptionEnabled ? 1 : 0);

	if (bEncryptionEnabled)
	{
		if (DTLSContext.IsValid())
		{
			if (InternalState == EDTLSHandlerState::Encrypted)
			{
				NewPacket.WriteBit(0); // handshake bit

				// Termination bit
				Packet.WriteBit(1);

				const int32 BytesWritten = SSL_write(DTLSContext->GetSSLPtr(), Packet.GetData(), Packet.GetNumBytes());
				if (BytesWritten <= 0)
				{
					LogError(TEXT("SSL_write"), BytesWritten);
					Packet.SetError();
					return;
				}
				else if (BytesWritten != Packet.GetNumBytes())
				{
					UE_LOGF(LogDTLSHandler, Warning, "Failed to write entire outgoing packet to SSL: %d / %lld", BytesWritten, Packet.GetNumBytes());
					Packet.SetError();
					return;
				}

				int32 Pending = BIO_ctrl_pending(DTLSContext->GetFilterBIO());
				if (Pending > 0)
				{
					if (Pending <= sizeof(TempBuffer))
					{
						const int32 BytesRead = BIO_read(DTLSContext->GetOutBIO(), TempBuffer, Pending);
						if (ensure(BytesRead == Pending))
						{
							NewPacket.SerializeBits(TempBuffer, BytesRead * 8);
						}
						else
						{
							UE_LOGF(LogDTLSHandler, Error, "BIO_read error: %d", BytesRead);
							Packet.SetError();
						}
					}
					else
					{
						UE_LOGF(LogDTLSHandler, Error, "BIO_ctrl_pending returned value out of range: %d", Pending);
						Packet.SetError();
					}
				}
				else
				{
					UE_LOGF(LogDTLSHandler, Error, "BIO_ctrl_pending error: %d", Pending);
					Packet.SetError();
				}
			}
			else if (InternalState == EDTLSHandlerState::Handshaking)
			{
				UE_LOGF(LogDTLSHandler, Verbose, "Attempted to send packet during handshaking, dropping.");
				Packet.Reset();
				return;
			}
			else
			{
				UE_LOGF(LogDTLSHandler, Log, "Attempted to send packet while handler was in invalid state");
				Packet.SetError();
			}
		}
		else
		{
			UE_LOGF(LogDTLSHandler, Verbose, "Outgoing: Invalid DTLS context.");
			Packet.SetError();
		}
	}
	else
	{
		NewPacket.SerializeBits(Packet.GetData(), Packet.GetNumBits());
	}

	if (!Packet.IsError())
	{
		Packet = MoveTemp(NewPacket);
	}
}

void FDTLSHandlerComponent::Tick(float DeltaTime)
{
	TickHandshake();
}

int32 FDTLSHandlerComponent::GetReservedPacketBits() const
{
	const int32 BlockSizeInBytes = 32;

	// Worst case includes the encryption bit, handshake bit, the termination bit, padding up to the next whole byte, a block of padding, and the header size
	return 3 + 7 + (BlockSizeInBytes * 8) + FMath::Max(DTLS1_RT_HEADER_LENGTH, DTLS1_HM_HEADER_LENGTH) * 8;
}

void FDTLSHandlerComponent::CountBytes(FArchive& Ar) const
{
	FEncryptionComponent::CountBytes(Ar);

	const SIZE_T SizeOfThis = sizeof(*this) - sizeof(FEncryptionComponent);
	Ar.CountBytes(SizeOfThis, SizeOfThis);
}

void FDTLSHandlerComponent::LogError(const TCHAR* Context, int32 Result)
{
	if (DTLSContext.IsValid())
	{
		const int32 SSLErrorCode = SSL_get_error(DTLSContext->GetSSLPtr(), Result);

		UE_LOGF(LogDTLSHandler, Error, "%ls: SSL_get_error: %d", Context, SSLErrorCode);

		BIO* MemBIO = BIO_new(BIO_s_mem());
		ERR_print_errors(MemBIO);

		char* ErrorBuffer = nullptr;
		size_t BufferLength = BIO_get_mem_data(MemBIO, &ErrorBuffer);

		FString ErrorString = FString::ConstructFromPtrSize(ErrorBuffer, BufferLength);

		BIO_free(MemBIO);

		UE_LOGF(LogDTLSHandler, Error, "%ls: ERR_print_errors: %ls", Context, *ErrorString);
	}
}

#endif // WITH_SSL