// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DTLSHandlerTypes.h"
#include "DTLSCertificate.h"

#define UE_API DTLSHANDLERCOMPONENT_API

#if WITH_SSL

/*
* Certificate store that can generate self-signed X509 certificates for DTLS
*/
struct FDTLSCertStore
{
public:
	/**
	 * Retrieve cert store
	 *
	 * @return singleton cert store reference
	 */
	static UE_API FDTLSCertStore& Get();

	/**
	 * Create a new certificate
	 *
	 * @Param Lifetime time in seconds until expiration of certificate
	 *
	 * @return shared pointer to certificate, valid if creation succeeded
	 */
	UE_API TSharedPtr<FDTLSCertificate> CreateCert(const FTimespan& Lifetime);

	/**
	 * Create a new certificate and store internally
	 *
	 * @Param Lifetime time in seconds until expiration of certificate
	 * @Param Identifier name to use when storing certificate for later use
	 *
	 * @return shared pointer to certificate, valid if creation succeeded
	 */
	UE_API TSharedPtr<FDTLSCertificate> CreateCert(const FTimespan& Lifetime, const FString& Identifier);

	/**
	 * Retrieve a certificate using unique identifier
	 *
	 * @Param Identifier unique identifier used when cert was added
	 *
	 * @return shared pointer to certificate, valid if it was found
	 */
	UE_API TSharedPtr<FDTLSCertificate> GetCert(const FString& Identifier) const;

	/**
	 * Import a certificate from file
	 *
	 * @Param CertPath path to certificate file
	 *
	 * @return shared pointer to certificate, valid if import succeeded
	 */
	UE_API TSharedPtr<FDTLSCertificate> ImportCert(const FString& CertPath) const;

	/**
	 * Import a certificate from file and store internally
	 *
	 * @Param CertPath path to certificate file
	 * @Param Identifier name to use when storing certificate for later use
	 *
	 * @return shared pointer to certificate, valid if import succeeded
	 */
	UE_API TSharedPtr<FDTLSCertificate> ImportCert(const FString& CertPath, const FString& Identifier);

	/**
	 * Remove a certificate using unique identifier
	 *
	 * @Param Identifier unique identifier used when cert was added
	 *
	 * @return true if a certificate was removed
	 */
	UE_API bool RemoveCert(const FString& Identifier);

private:
	TMap<FString, TSharedPtr<FDTLSCertificate>> CertMap;

	static TUniquePtr<FDTLSCertStore> Instance;
};

#endif // WITH_SSL

#undef UE_API
