// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DTLSHandlerTypes.h"
#include "Misc/Timespan.h"

#define UE_API DTLSHANDLERCOMPONENT_API

#if WITH_SSL

/*
* Wrapper for a fingerprint (SHA256 hash) of an X509 certificate
*/
struct FDTLSFingerprint
{
public:
	/** SHA256 hash length in bytes */
	static constexpr uint32 Length = 32;

	FDTLSFingerprint()
	{
		Reset();
	}

	FDTLSFingerprint(const FDTLSFingerprint&) = delete;
	FDTLSFingerprint& operator=(const FDTLSFingerprint&) = delete;

	/** Zero the fingerprint */
	void Reset()
	{
		FMemory::Memzero(Data, Length);
	}
	
	/** Get array view of fingerprint data */
	TArrayView<const uint8> GetData() const { return MakeArrayView(Data, FDTLSFingerprint::Length); }

	uint8 Data[Length];
};

/*
* Container for an X509 certificate
*/
struct FDTLSCertificate
{
public:
	UE_API FDTLSCertificate();
	UE_API ~FDTLSCertificate();

	FDTLSCertificate(const FDTLSCertificate&) = delete;
	FDTLSCertificate& operator=(const FDTLSCertificate&) = delete;
	FDTLSCertificate(FDTLSCertificate&&) = delete;
	FDTLSCertificate& operator=(FDTLSCertificate&&) = delete;

	/** Get OpenSSL private key pointer */
	EVP_PKEY* GetPKey() const { return PKey; }

	/** Get OpenSSL X509 certificate pointer */
	X509* GetCertificate() const { return Certificate; }

	/** Get array view of fingerprint data */
	TArrayView<const uint8> GetFingerprint() const { return Fingerprint.GetData(); }

	/**
	 * Generate a self-signed certificate
	 *
	 * @param Lifetime number of seconds until the certificate should expire
	 * @return true if creation succeeded
	 */
	UE_API bool GenerateCertificate(const FTimespan& Lifetime);

	/**
	 * Export current certificate to PEM file format
	 *
	 * @param CertPath path to output file
	 * @return true if export succeeded
	 */
	UE_API bool ExportCertificate(const FString& CertPath);

	/**
	 * Import certificate from PEM file format
	 *
	 * @param CertPath path to input file
	 * @return true if import succeeded
	 */
	UE_API bool ImportCertificate(const FString& CertPath);

private:
	UE_API void FreeCertificate();
	UE_API bool GenerateFingerprint();

	EVP_PKEY* PKey;
	X509* Certificate;
	FDTLSFingerprint Fingerprint;
};

#endif // WITH_SSL

#undef UE_API
