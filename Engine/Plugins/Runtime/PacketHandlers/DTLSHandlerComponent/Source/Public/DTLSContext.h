// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DTLSHandlerTypes.h"
#include "DTLSCertificate.h"

#if WITH_SSL

class FDTLSHandlerComponent;

/*
* Wrapper for a pre-shared key value to be used instead of self-signed certificates
*/
struct FDTLSPreSharedKey
{
public:
	FDTLSPreSharedKey() {}

	FDTLSPreSharedKey(const FDTLSPreSharedKey&) = delete;
	FDTLSPreSharedKey& operator=(const FDTLSPreSharedKey&) = delete;

	TArrayView<const uint8> GetKey() const 
	{ 
		return MakeArrayView(Key); 
	}
	
	const FString& GetIdentity() const 
	{ 
		return Identity; 
	}
	
	void SetPreSharedKey(TArrayView<const uint8> KeyView)
	{
		Key.Append(KeyView.begin(), KeyView.Num());
	}

	void SetIdentity(const FString& InIdentity)
	{
		Identity = InIdentity;
	}

private:
	TArray<uint8> Key;
	FString Identity;
};

/*
* OpenSSL BIO applied to SSL connection to enforce MTU for handshake packets
*/
struct FMTUFilter
{
public:
	static int Create(BIO* BIOPtr);
	static int Destroy(BIO* BIOPtr);
	static int Write(BIO* BIOPtr, const char* Data, int DataLen);
	static long Ctrl(BIO* BIOPtr, int CtrlCommand, long, void*);

public:
	TArray<int32, TInlineAllocator<16>> FragmentSizes;
};

/*
* Context data for OpenSSL connection setup
*/
struct FDTLSContext
{
public:
	FDTLSContext() = delete;
	FDTLSContext(EDTLSContextType InContextType);
	~FDTLSContext();

	/**
	 * Initialize the context
	 *
	 * @param MaxPacketSize the MTU to respect when processing handshake packets
	 * @param CertId unique identifier of certificate to use, retrieved from FDTLSCertStore
	 * @param Handler the packet handler component that owns this context
	 *
	 * @return false if initialization process failed for any reason
	 */
	bool Initialize(const int32 MaxPacketSize, const FString& CertId, FDTLSHandlerComponent* Handler);

	// Get OpenSSL SSL pointer
	SSL* GetSSLPtr() const { return SSLPtr; }

	// Get OpenSSL input BIO pointer
	BIO* GetInBIO() const { return InBIO; }

	// Get OpenSSL output BIO pointer
	BIO* GetOutBIO() const { return OutBIO; }

	// Get OpenSSL custom filter BIO pointer
	BIO* GetFilterBIO() const { return FilterBIO; }

	/**
	 * Query for context type
	 *
	 * @return EDTLSContextType context type
	 */
	EDTLSContextType GetContextType() const { return ContextType; }

	/**
	 * Query for handshaking status
	 *
	 * @return true if the DTLS handshake has completed
	 */
	bool IsHandshakeComplete() const;

private:
	EDTLSContextType ContextType;

	SSL_CTX* SSLContext;
	SSL* SSLPtr;
	BIO* InBIO;
	BIO* OutBIO;
	BIO* FilterBIO;
	BIO_METHOD* MTUFilterMethod;

	TSharedPtr<FDTLSCertificate> Cert;
};

#endif // WITH_SSL