// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "PacketHandler.h"
#include "EncryptionComponent.h"
#include "UObject/CoreNet.h"
#include "DTLSContext.h"

#define UE_API DTLSHANDLERCOMPONENT_API

#if WITH_SSL

extern TAutoConsoleVariable<int32> CVarPreSharedKeys;

/*
* DTLS encryption component.
*/
class FDTLSHandlerComponent : public FEncryptionComponent
{
public:
	UE_API FDTLSHandlerComponent();
	UE_API virtual ~FDTLSHandlerComponent();

	UE_API virtual void SetEncryptionData(const FEncryptionData& EncryptionData) override;

	// After calling this, future outgoing packets will be encrypted (until a call to DisableEncryption).
	UE_API virtual void EnableEncryption() override;

	// After calling this, future outgoing packets will not be encrypted (until a call to DisableEncryption).
	UE_API virtual void DisableEncryption() override;

	// Returns true if encryption is currently enabled.
	UE_API virtual bool IsEncryptionEnabled() const override;

	// HandlerComponent interface
	UE_API virtual void Initialize() override;
	UE_API virtual bool IsValid() const override;
	UE_API virtual void Incoming(FBitReader& Packet) override;
	UE_API virtual void Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits) override;
	UE_API virtual int32 GetReservedPacketBits() const override;
	UE_API virtual void CountBytes(FArchive& Ar) const override;

	UE_API virtual void Tick(float DeltaTime) override;

	enum class EDTLSHandlerState
	{
		Unencrypted,
		Handshaking,
		Encrypted,
	};

	// Retrieve pre shared key, if set
	const FDTLSPreSharedKey* GetPreSharedKey() const { return PreSharedKey.Get(); }

	// Retrieve expected remote certificate fingerprint, if set
	const FDTLSFingerprint* GetRemoteFingerprint() const { return RemoteFingerprint.Get(); }

private:
	// Process DTLS handshake
	UE_API void TickHandshake();
	UE_API void DoHandshake();

	UE_API void LogError(const TCHAR* Context, int32 Result);

private:
	EDTLSHandlerState InternalState;

	TUniquePtr<FDTLSContext> DTLSContext;
	TUniquePtr<FDTLSPreSharedKey> PreSharedKey;
	TUniquePtr<FDTLSFingerprint> RemoteFingerprint;

	FString CertId;

	uint8 TempBuffer[MAX_PACKET_SIZE];

	bool bPendingHandshakeData;
};

#endif // WITH_SSL

/**
 * The public interface to this module.
 */
class FDTLSHandlerComponentModule : public FPacketHandlerComponentModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/* Creates an instance of this component */
	virtual TSharedPtr<HandlerComponent> CreateComponentInstance(FString& Options) override;
};

#undef UE_API
