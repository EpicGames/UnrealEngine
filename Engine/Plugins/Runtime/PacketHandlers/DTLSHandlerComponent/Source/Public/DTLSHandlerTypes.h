// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_SSL

/** Forward some OpenSSL types  */

struct evp_pkey_st;
typedef struct evp_pkey_st EVP_PKEY;

struct x509_st;
typedef struct x509_st X509;

struct ssl_ctx_st;
typedef struct ssl_ctx_st SSL_CTX;

struct ssl_st;
typedef struct ssl_st SSL;

struct bio_st;
typedef struct bio_st BIO;

typedef struct bio_method_st BIO_METHOD;

enum class EDTLSContextType
{
	Server,
	Client
};

const TCHAR* LexToString(EDTLSContextType ContextType);

#endif // WITH_SSL

DECLARE_LOG_CATEGORY_EXTERN(LogDTLSHandler, Log, All);
